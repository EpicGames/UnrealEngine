{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Automation Utilities",
	"Description": "Tools and Utilities for Automation purposes",
	"Category": "Testing",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AutomationUtils",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		},
		{
			"Name": "AutomationUtilsEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"DocsURL": "",
	"IsExperimentalVersion": false
}