// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationUtils : ModuleRules
	{
		public AutomationUtils(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Json",
					"RHI",
					"RenderCore"
                }
			);
		}
	}
}
