// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

//Module Implementation Call
IMPLEMENT_MODULE(FDefaultModuleImpl, AutomationUtils)

