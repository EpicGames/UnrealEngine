// Copyright Epic Games, Inc. All Rights Reserved.

#include "AutomationUtilsBlueprintLibrary.h"
#include "AssetCompilingManager.h"
#include "SceneViewExtension.h"
#include "Engine/Level.h"
#include "SceneView.h"
#include "ContentStreaming.h"
#include "Engine/Texture.h"
#include "ImageUtils.h"
#include "Misc/EngineVersion.h"
#include "HardwareInfo.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Scalability.h"
#include "Engine/GameViewportClient.h"
#include "Misc/FileHelper.h"
#include "Misc/FileHelper.h"
#include "RenderingThread.h"
#include "Serialization/JsonSerializer.h"
#include "Engine/GameEngine.h"
#include "Materials/MaterialInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AutomationUtilsBlueprintLibrary)

//Private Helper Class Definitions
class FAutomationUtilsGameplayViewExtension : public FSceneViewExtensionBase
{
public:
	FAutomationUtilsGameplayViewExtension(const FAutoRegister& AutoRegister)
		: FSceneViewExtensionBase(AutoRegister)
	{
	}

	void SetupViewFamily(FSceneViewFamily& InViewFamily) override
	{
		// Turn off common show flags for noisy sources of rendering.
		FEngineShowFlags& ShowFlags = InViewFamily.EngineShowFlags;
		ShowFlags.DisableAdvancedFeatures();
		ShowFlags.SetAntiAliasing(0);
		ShowFlags.SetMotionBlur(0);
		ShowFlags.SetScreenSpaceAO(0);
		ShowFlags.SetContactShadows(0);
		ShowFlags.SetBloom(0);

		// Turn off time the ultimate source of noise.
		InViewFamily.Time = FGameTime();
	}

	bool IsActiveThisFrame_Internal(const FSceneViewExtensionContext&) const
	{
		return true;
	}

	/** We always want to go last. */
	virtual int32 GetPriority() const override { return MIN_int32; }
};



class FAutomationUtilsGameplayAutomationScreenshotInstance
{
public:
	FAutomationUtilsGameplayAutomationScreenshotInstance(FString InScreenshotName, float MaxGlobalError, float MaxLocalError, FString MapNameOverride = TEXT(""));
	~FAutomationUtilsGameplayAutomationScreenshotInstance();

public:
	void HandleScreenshotData(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData);
	void WorldDestroyed(ULevel* InLevel, UWorld* InWorld);

private:
	void Unbind();

private:
	FString ScreenshotName;
	FString MetadataJsonString;
	FString DeterminedPath;
	FString MapName;
	TWeakObjectPtr<UWorld> World;
	TSharedPtr< class FAutomationUtilsGameplayViewExtension, ESPMode::ThreadSafe > AutomationViewExtension;
};



class FAutomationUtilsGameplayAutomationScreenshotFactory
{
public:
	static bool CreateScreenshotInstance(const FString ScreenshotName, float MaxGlobalError, float MaxLocalError, FString MapNameOverride);
	static bool RequestDeleteScreenshotInstance(const FString ScreenshotName);

private:
	static TMap<FString, FAutomationUtilsGameplayAutomationScreenshotInstance*> ManagedScreenshotInstances;
};



FAutomationUtilsGameplayAutomationScreenshotInstance::FAutomationUtilsGameplayAutomationScreenshotInstance(FString InScreenshotName, float MaxGlobalError, float MaxLocalError, FString MapNameOverride)
	: ScreenshotName(*InScreenshotName)
	, MetadataJsonString(TEXT("{}"))
	, DeterminedPath(TEXT(""))
	, MapName(MapNameOverride.IsEmpty() ? GWorld->GetName() : MapNameOverride)
	, World(GWorld.GetReference())
{
	if (GEngine && GEngine->GameViewport)
	{
		//FlushRendering
		FlushRenderingCommands();
		//hook to the screenshot delegate
		GEngine->GameViewport->OnScreenshotCaptured().AddRaw(this, &FAutomationUtilsGameplayAutomationScreenshotInstance::HandleScreenshotData);
		//And a removed-from-world delegate too, just in case
		FWorldDelegates::LevelRemovedFromWorld.AddRaw(this, &FAutomationUtilsGameplayAutomationScreenshotInstance::WorldDestroyed);

		//Generate Json Metadata relevant to rendering device, quality settings, and comparison tolerances
		TSharedPtr<FJsonObject> JsonObject = MakeShareable(new FJsonObject);
		//General Stuff
		JsonObject->SetStringField(TEXT("screenShotName"), FPaths::MakeValidFileName(ScreenshotName, TEXT('_')));
		JsonObject->SetStringField(TEXT("context"), MapName);
		JsonObject->SetStringField(TEXT("id"), FGuid::NewGuid().ToString());
		JsonObject->SetStringField(TEXT("Commit"), FEngineVersion::Current().HasChangelist() ? FString::FromInt(FEngineVersion::Current().GetChangelist()) : FString(TEXT("")));
		FVector2D ViewportSize;  //Width and Height
		GEngine->GameViewport->GetViewportSize(ViewportSize);
		JsonObject->SetNumberField(TEXT("width"), ViewportSize.X);
		JsonObject->SetNumberField(TEXT("height"), ViewportSize.Y);
		//RHI
		JsonObject->SetStringField(TEXT("platform"), FPlatformProperties::IniPlatformName());
		JsonObject->SetStringField(TEXT("rhi"), FHardwareInfo::GetHardwareInfo(NAME_RHI));
		FString FeatureLevel;
		GetFeatureLevelName(GMaxRHIFeatureLevel, FeatureLevel);
		JsonObject->SetStringField(TEXT("featureLevel"), FeatureLevel);
		JsonObject->SetBoolField(TEXT("bIsStereo"), GEngine->StereoRenderingDevice.IsValid() ? GEngine->StereoRenderingDevice->IsStereoEnabled() : false);
		//Vendor
		JsonObject->SetStringField(TEXT("vendor"), RHIVendorIdToString());
		JsonObject->SetStringField(TEXT("adapterName"), GRHIAdapterName);
		JsonObject->SetStringField(TEXT("adapterInternalDriverVersion"), GRHIAdapterInternalDriverVersion);
		JsonObject->SetStringField(TEXT("adapterUserDriverVersion"), GRHIAdapterUserDriverVersion);
		JsonObject->SetStringField(TEXT("uniqueDeviceId"), FPlatformMisc::GetDeviceId());
		//Quality
		Scalability::FQualityLevels QualityLevels = Scalability::GetQualityLevels();
		JsonObject->SetNumberField(TEXT("resolutionQuality"), QualityLevels.ResolutionQuality);
		JsonObject->SetNumberField(TEXT("viewDistanceQuality"), QualityLevels.ViewDistanceQuality);
		JsonObject->SetNumberField(TEXT("antiAliasingQuality"), QualityLevels.AntiAliasingQuality);
		JsonObject->SetNumberField(TEXT("shadowQuality"), QualityLevels.ShadowQuality);
		JsonObject->SetNumberField(TEXT("globalIlluminationQuality"), QualityLevels.GlobalIlluminationQuality);
		JsonObject->SetNumberField(TEXT("reflectionQuality"), QualityLevels.ReflectionQuality);
		JsonObject->SetNumberField(TEXT("postProcessQuality"), QualityLevels.PostProcessQuality);
		JsonObject->SetNumberField(TEXT("textureQuality"), QualityLevels.TextureQuality);
		JsonObject->SetNumberField(TEXT("effectsQuality"), QualityLevels.EffectsQuality);
		JsonObject->SetNumberField(TEXT("foliageQuality"), QualityLevels.FoliageQuality);
		JsonObject->SetNumberField(TEXT("shadingQuality"), QualityLevels.ShadingQuality);
		//ComparisonOptions
		JsonObject->SetBoolField(TEXT("bHasComparisonRules"), true);
		JsonObject->SetNumberField(TEXT("toleranceRed"), 8);
		JsonObject->SetNumberField(TEXT("toleranceGreen"), 8);
		JsonObject->SetNumberField(TEXT("toleranceBlue"), 8);
		JsonObject->SetNumberField(TEXT("toleranceAlpha"), 8);
		JsonObject->SetNumberField(TEXT("toleranceMinBrightness"), 0);
		JsonObject->SetNumberField(TEXT("toleranceMaxBrightness"), 255);
		JsonObject->SetNumberField(TEXT("maximumLocalError"), MaxLocalError);
		JsonObject->SetNumberField(TEXT("maximumGlobalError"), MaxGlobalError);
		JsonObject->SetBoolField(TEXT("bIgnoreAntiAliasing"), true);
		JsonObject->SetBoolField(TEXT("bIgnoreColors"), false);



		//Serialize to String
		TSharedRef<TJsonWriter<>> Writer = TJsonWriterFactory<>::Create(&MetadataJsonString);
		FJsonSerializer::Serialize(JsonObject.ToSharedRef(), Writer);
		GLog->Log(FString::Printf(TEXT("Gameplay Automation Screenshot Metadata Serialized to %d characters"), MetadataJsonString.Len()));



		//Output path for both screenshot image and metadata json
		DeterminedPath = FPaths::AutomationDir() / TEXT("Incoming") / MapName / ScreenshotName / FPlatformProperties::IniPlatformName();

		//we parse some stuff out of HardwareDetailsString and make a pretty folder name
		FString HardwareDetailsString;
		FString HardwareDetails = FHardwareInfo::GetHardwareDetailsString();

		FString RHIString;
		FString RHILookup = NAME_RHI.ToString() + TEXT("=");
		if (FParse::Value(*HardwareDetails, *RHILookup, RHIString))
		{
			HardwareDetailsString = (HardwareDetailsString + TEXT("_")) + RHIString;
		}

		FString TextureFormatString;
		FString TextureFormatLookup = NAME_TextureFormat.ToString() + TEXT("=");
		if (FParse::Value(*HardwareDetails, *TextureFormatLookup, TextureFormatString))
		{
			HardwareDetailsString = (HardwareDetailsString + TEXT("_")) + TextureFormatString;
		}

		FString DeviceTypeString;
		FString DeviceTypeLookup = NAME_DeviceType.ToString() + TEXT("=");
		if (FParse::Value(*HardwareDetails, *DeviceTypeLookup, DeviceTypeString))
		{
			HardwareDetailsString = (HardwareDetailsString + TEXT("_")) + TextureFormatString;
		}

		//Also add FeatureLevel from earlier
		HardwareDetailsString = (HardwareDetailsString + TEXT("_")) + FeatureLevel;

		if (HardwareDetailsString.Len() > 0)
		{
			//remove leading "_"
			HardwareDetailsString.RightChopInline(1, EAllowShrinking::No);
		}

		// We need a unique ID for filenames from this run. We used to use GetDeviceId() but that is not guaranteed to return
		// a valid string on some platforms.
		FString DeviceIdString = FPlatformMisc::GetDeviceId();
		if (DeviceIdString.IsEmpty())
		{
			DeviceIdString = FGuid::NewGuid().ToString(EGuidFormats::Short).ToLower();
		}

		//now plop that back onto the path we're building
		DeterminedPath = DeterminedPath / HardwareDetailsString;
		DeterminedPath = DeterminedPath / DeviceIdString + TEXT(".png");

		//Remove as many noisy rendering conditions as we can until the screenshot has been taken
		AutomationViewExtension = FSceneViewExtensions::NewExtension<FAutomationUtilsGameplayViewExtension>();

		GLog->Log(FString::Printf(TEXT("Determined Path for screenshot \"%s\" to be %s"), *ScreenshotName, *DeterminedPath));
	}
	else
	{
		FTSTicker::GetCoreTicker().AddTicker(TEXT("FAutomationUtilsGameplayScreenshotInstanceAutoCleanup"), 0.1f, [this](float)
		{
			QUICK_SCOPE_CYCLE_COUNTER(STAT_FAutomationUtilsGameplayScreenshotInstanceAutoCleanup);
			FAutomationUtilsGameplayAutomationScreenshotFactory::RequestDeleteScreenshotInstance(ScreenshotName);
			return false;
		});
	}
}

FAutomationUtilsGameplayAutomationScreenshotInstance::~FAutomationUtilsGameplayAutomationScreenshotInstance()
{
	Unbind();
}

void FAutomationUtilsGameplayAutomationScreenshotInstance::HandleScreenshotData(int32 InSizeX, int32 InSizeY, const TArray<FColor>& InImageData)
{
	check(IsInGameThread());


	GLog->Log(FString::Printf(TEXT("Gameplay Automation Screenshot \"%s\" taken with size: %d x %d"), *ScreenshotName, InSizeX, InSizeY));

	//create directory if it doesn't exist
	IFileManager::Get().MakeDirectory(*FPaths::GetPath(DeterminedPath), true);

	//Save Image File
	TArray64<uint8> CompressedBitmap;
	FImageUtils::PNGCompressImageArray(InSizeX, InSizeY, TArrayView64<const FColor>(InImageData.GetData(), InImageData.Num()), CompressedBitmap);
	FFileHelper::SaveArrayToFile(CompressedBitmap, *DeterminedPath);
	GLog->Log(FString::Printf(TEXT("Saved %" INT64_FMT " bytes of screenshot image to %s"), CompressedBitmap.Num(), *DeterminedPath));

	//Save Metadata Json
	FString MetadataPath = FPaths::ChangeExtension(DeterminedPath, TEXT("json"));
	FFileHelper::SaveStringToFile(MetadataJsonString, *MetadataPath, FFileHelper::EEncodingOptions::ForceUTF8WithoutBOM);
	GLog->Log(FString::Printf(TEXT("Saved %d bytes of metadata json to %s"), MetadataJsonString.Len(), *MetadataPath));

	//remove our rendering options
	AutomationViewExtension.Reset();

	//deallocate this object, it's done its part
	FAutomationUtilsGameplayAutomationScreenshotFactory::RequestDeleteScreenshotInstance(ScreenshotName);
}

void FAutomationUtilsGameplayAutomationScreenshotInstance::WorldDestroyed(ULevel* InLevel, UWorld* InWorld)
{
	if (InLevel == nullptr && InWorld == World.Get())
	{
		GLog->Log(FString::Printf(TEXT("Screenshot \"%s\" skipped - level was removed from world before we got our screenshot"), *ScreenshotName));
		//nothing left for this object to do (we missed the timing somehow), so clean up
		FAutomationUtilsGameplayAutomationScreenshotFactory::RequestDeleteScreenshotInstance(ScreenshotName);
	}
}

void FAutomationUtilsGameplayAutomationScreenshotInstance::Unbind()
{
	if (GEngine && GEngine->GameViewport)
	{
		GEngine->GameViewport->OnScreenshotCaptured().RemoveAll(this);
	}
	FWorldDelegates::LevelRemovedFromWorld.RemoveAll(this);
}


//FAutomationUtilsGameplayAutomationScreenshotFactory member definitions
bool FAutomationUtilsGameplayAutomationScreenshotFactory::CreateScreenshotInstance(const FString ScreenshotName, float MaxGlobalError, float MaxLocalError, FString MapNameOverride)
{
	if (ManagedScreenshotInstances.Contains(ScreenshotName))
	{
		GLog->Log(FString::Printf(TEXT("Cannot create GameplayScreenshotInstance - One with the name \"%s\" already exists!"), *ScreenshotName));
		return false;
	}
	else if (!(GEngine && GEngine->GameViewport))
	{
		//there's no GEngine, or else no game viewport to take a screenshot in
		GLog->Log(TEXT("Cannot create GameplayScreenshotInstance - either GEngine or GameViewport is null!"));
		return false;
	}
	else
	{
		ManagedScreenshotInstances.Add(ScreenshotName, new FAutomationUtilsGameplayAutomationScreenshotInstance(ScreenshotName, MaxGlobalError, MaxLocalError, MapNameOverride));
		GLog->Log(FString::Printf(TEXT("Created GameplayScreenshotInstance \"%s\""), *ScreenshotName));
		return true;
	}
}

bool FAutomationUtilsGameplayAutomationScreenshotFactory::RequestDeleteScreenshotInstance(const FString ScreenshotName)
{
	if (!ManagedScreenshotInstances.Contains(ScreenshotName))
	{
		GLog->Log(ELogVerbosity::Error, FString::Printf(TEXT("Tried to remove GameplayScreenshotInstance \"%s\", but it did not exist!"), *ScreenshotName));
		return false;
	}
	else
	{
		delete ManagedScreenshotInstances[ScreenshotName];
		ManagedScreenshotInstances.Remove(ScreenshotName);
		GLog->Log(FString::Printf(TEXT("Removed GameplayScreenshotInstance \"%s\""), *ScreenshotName));
		return true;
	}
}

TMap<FString, FAutomationUtilsGameplayAutomationScreenshotInstance*> FAutomationUtilsGameplayAutomationScreenshotFactory::ManagedScreenshotInstances;





//Blueprint Library
UAutomationUtilsBlueprintLibrary::UAutomationUtilsBlueprintLibrary(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

void UAutomationUtilsBlueprintLibrary::TakeGameplayAutomationScreenshot(const FString ScreenshotName, float MaxGlobalError, float MaxLocalError, FString MapNameOverride)
{
	FlushAsyncLoading();

	UWorld* CurrentWorld{ nullptr };
	// Make sure we finish all level streaming
	if (UGameEngine* GameEngine = Cast<UGameEngine>(GEngine))
	{
		if (UWorld* GameWorld = GameEngine->GetGameWorld())
		{
			CurrentWorld = GameWorld;
			GameWorld->FlushLevelStreaming(EFlushLevelStreamingType::Full);
		}
	}

	//Finish Loading Before Screenshot
	if (!FPlatformProperties::RequiresCookedData())
	{
		UMaterialInterface::SubmitRemainingJobsForWorld(CurrentWorld);
		//Finish Compiling all shaders and other async assets
		FAssetCompilingManager::Get().FinishAllCompilation();
	}

	//Stream in everything
	IStreamingManager::Get().StreamAllResources(0.0f);

	//Force all mip maps to load
	UTexture::ForceUpdateTextureStreaming();

	//Allocate automation object
	if (!FAutomationUtilsGameplayAutomationScreenshotFactory::CreateScreenshotInstance(ScreenshotName, MaxGlobalError, MaxLocalError, MapNameOverride))
	{
		GLog->Log(ELogVerbosity::Error, FString::Printf(TEXT("Failed to create GameplayAutomationScreenshot Instance for \"%s\"!"), *ScreenshotName));
	}

	//Actually Take Screenshot
	FScreenshotRequest::RequestScreenshot(ScreenshotName, false, true);
}

void UAutomationUtilsBlueprintLibrary::AddAutomationMountPoint(const FString& RootPath)
{
	FPackageName::RegisterMountPoint(RootPath, FPaths::AutomationTransientDir());
}

void UAutomationUtilsBlueprintLibrary::RemoveAutomationMountPoint(const FString& RootPath)
{
	FPackageName::UnRegisterMountPoint(RootPath, FPaths::AutomationTransientDir());
}

void UAutomationUtilsBlueprintLibrary::FinishAllAssetCompilation()
{
	// FinishAllCompilation drains the compile workers; FlushRenderingCommands then
	// drains the completion render commands those workers posted, so callers can
	// safely destroy compiled assets without racing against ~FMaterial.
	FAssetCompilingManager::Get().FinishAllCompilation();
	FlushRenderingCommands();
}



