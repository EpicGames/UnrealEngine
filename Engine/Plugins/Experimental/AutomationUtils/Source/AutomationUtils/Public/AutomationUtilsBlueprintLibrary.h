// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Kismet/BlueprintFunctionLibrary.h"


#include "AutomationUtilsBlueprintLibrary.generated.h"

#define UE_API AUTOMATIONUTILS_API

UCLASS(MinimalAPI)
class UAutomationUtilsBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void TakeGameplayAutomationScreenshot(const FString ScreenshotName, float MaxGlobalError = .02, float MaxLocalError = .12, FString MapNameOverride = TEXT(""));

	/*
	 * Registers a virtual mount point rooted at FPaths::AutomationTransientDir().
	 * Call RemoveAutomationMountPoint to unregister when done.
	 * @param RootPath  Virtual root path.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void AddAutomationMountPoint(const FString& RootPath = TEXT("/Automation/"));

	/*
	 * Unregisters a mount point previously registered with AddAutomationMountPoint.
	 * @param RootPath  Virtual root path.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void RemoveAutomationMountPoint(const FString& RootPath = TEXT("/Automation/"));

	/*
	 * Blocks the game thread until all in-flight asset compilation finishes and
	 * the render thread has drained the followup commands the compile workers
	 * posted. Use this in test teardown before deleting assets whose shader
	 * compilation might still be in flight; otherwise GC can destroy an FMaterial
	 * mid-compile and hit ~FMaterial's RenderingThreadCompilingShaderMapId == 0 assert.
	 */
	UFUNCTION(BlueprintCallable, Category = "Automation")
	static UE_API void FinishAllAssetCompilation();
};

#undef UE_API
