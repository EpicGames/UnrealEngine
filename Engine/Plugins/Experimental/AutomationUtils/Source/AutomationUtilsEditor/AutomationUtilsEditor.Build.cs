// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AutomationUtilsEditor : ModuleRules
	{
		public AutomationUtilsEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AutomationMessages",
					"Core",
					"CoreUObject",
					"Engine",
					"Json",
					"JsonUtilities",
					"ScreenShotComparisonTools",
				}
			);
		}
	}
}
