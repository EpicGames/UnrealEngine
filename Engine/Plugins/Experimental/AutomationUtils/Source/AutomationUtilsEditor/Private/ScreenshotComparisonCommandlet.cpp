// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScreenshotComparisonCommandlet.h"
#include "ImageComparer.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/IScreenShotToolsModule.h"
#include "Misc/FileHelper.h"
#include "AutomationWorkerMessages.h"
#include "JsonObjectConverter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ScreenshotComparisonCommandlet)


DEFINE_LOG_CATEGORY(LogScreenshotComparison);

UScreenshotComparisonCommandlet::UScreenshotComparisonCommandlet(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
}

int32 UScreenshotComparisonCommandlet::Main(const FString& CmdLineParameters)
{
	FString IncomingPath = FPaths::AutomationDir() / TEXT("Incoming/");
	FString GroundTruthPath = FPaths::ProjectDir() / TEXT("Test") / TEXT("Screenshots/");

	TArray<FString> MapList;
	FString MapListStr;
	if (FParse::Value(*CmdLineParameters, TEXT("Maps="), MapListStr))
	{
		MapListStr.ParseIntoArray(MapList, TEXT("+"), true);
	}

	UE_LOGF(LogScreenshotComparison, Log, "Incoming Path: %ls", *IncomingPath);
	UE_LOGF(LogScreenshotComparison, Log, "Ground Truth Path: %ls", *GroundTruthPath);

	//Get All Screenshots in Incoming
	TArray<FString> FilesToCompare;
	if (FPaths::DirectoryExists(IncomingPath))
	{
		FString AbsoluteIncomingPath = IncomingPath;
		FPaths::MakePathRelativeTo(AbsoluteIncomingPath, *FPaths::RootDir());
		AbsoluteIncomingPath = FPaths::RootDir() / AbsoluteIncomingPath;

		
		if (MapList.Num())
		{
			for (FString LevelName : MapList)
			{
				TArray<FString> LevelFiles;
				IFileManager::Get().FindFilesRecursive(LevelFiles, *(AbsoluteIncomingPath / LevelName), TEXT("*.png"), true, false);
				FilesToCompare.Append(LevelFiles);
			}
		}
		else
		{
			//No Restrictions -> get ALL images
			IFileManager::Get().FindFilesRecursive(FilesToCompare, *AbsoluteIncomingPath, TEXT("*.png"), true, false);
		}
	}

	if (FilesToCompare.Num())
	{
		UE_LOGF(LogScreenshotComparison, Log, "Comparing %d screenshots", FilesToCompare.Num());
	}
	else
	{
		UE_LOGF(LogScreenshotComparison, Warning, "Found no screenshots (*.png) in IncomingPath %ls", *IncomingPath);
	}



	//Do comparison
	IScreenShotToolsModule& ScreenShotModule = FModuleManager::LoadModuleChecked<IScreenShotToolsModule>("ScreenShotComparisonTools");
	IScreenShotManagerPtr ScreenshotManager = ScreenShotModule.GetScreenShotManager();
	int32 New = 0;
	int32 Fails = 0;
	int32 Passes = 0;
	for (FString ScreenshotName : FilesToCompare)
	{

		FString ApprovedMetadataFile = FPaths::ChangeExtension(ScreenshotName, ".json");

		FString Json;
		if (FFileHelper::LoadFileToString(Json, *ApprovedMetadataFile))
		{
			FAutomationScreenshotMetadata Metadata;
			if (FJsonObjectConverter::JsonObjectStringToUStruct(Json, &Metadata, 0, 0))
			{

				// #agrant (@todo) do we really want to keep the image? It results in a duplicate in the generated report
				FImageComparisonResult Result = ScreenshotManager->CompareScreenshotAsync(ScreenshotName, Metadata, EScreenShotCompareOptions::KeepImage).Get();
				if (Result.IsNew())
				{
					UE_LOGF(LogScreenshotComparison, Warning, "Incoming file %ls is new", *Result.IncomingFilePath);
					New++;
				}
				else if (Result.AreSimilar())
				{
					UE_LOGF(LogScreenshotComparison, Log, "Incoming file %ls is similar!  (Global %f, Local %f)",
						*Result.IncomingFilePath, Result.GlobalDifference, Result.MaxLocalDifference);
					Passes++;
				}
				else
				{
					UE_LOGF(LogScreenshotComparison, Error, "Incoming file %ls is different! (Global %f, Local %f) : %ls",
						*Result.IncomingFilePath, Result.GlobalDifference, Result.MaxLocalDifference, *Result.ErrorMessage.ToString());
					Fails++;
				}
			}
			else
			{
				UE_LOGF(LogScreenshotComparison, Error, "Failed to parse JSON for %ls into screenshot meta data", *ScreenshotName);
			}

		}
		else
		{
			UE_LOGF(LogScreenshotComparison, Error, "Failed to load JSON for %ls", *ScreenshotName);
		}
	}

	int32 Total = New + Passes + Fails;
	if (Total)
	{
		UE_LOGF(LogScreenshotComparison, Log, "Comparison Complete!  (New: %d (%.2f%%), Fail: %d (%.2f%%), Pass: %d (%.2f%%)",
			New, (float)New / Total * 100, Fails, (float)Fails / Total * 100, Passes, (float)Passes / Total * 100);
	}
	else
	{
		UE_LOGF(LogScreenshotComparison, Log, "Comparison Complete!");
	}


	return 0;
}
