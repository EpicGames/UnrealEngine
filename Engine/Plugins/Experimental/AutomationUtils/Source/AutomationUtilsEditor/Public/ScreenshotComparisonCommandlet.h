// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Commandlets/Commandlet.h"
#include "ScreenshotComparisonCommandlet.generated.h"

DECLARE_LOG_CATEGORY_EXTERN(LogScreenshotComparison, Log, All);

UCLASS(config = Editor)
class UScreenshotComparisonCommandlet : public UCommandlet
{
	GENERATED_UCLASS_BODY()

public:

	//~ Begin UCommandlet Interface
	virtual int32 Main(const FString& CmdLineParams) override;
	//~ End UCommandlet Interface
};
